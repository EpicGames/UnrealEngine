{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Mover Integrations",
	"Description": "Mover Integrations is a Unreal Engine plugin acting as an umbrella to cover a variety of modules supporting Mover's integration with other plugins, such as animation, AI, and other gameplay systems.",
	"Category": "Other",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "MoverIntegrations",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "MoverMassIntegration",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "Mover",
			"Enabled": true
		},
		{
			"Name": "MassGameplay",
			"Enabled": true
		}
	]
}