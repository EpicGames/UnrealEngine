// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

/**
 * Module that serves as umbrella of a variety of integration modules between Mover and other systems (such as animation, AI, and other gameplay systems)
 */
class FMoverIntegrationsModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};
