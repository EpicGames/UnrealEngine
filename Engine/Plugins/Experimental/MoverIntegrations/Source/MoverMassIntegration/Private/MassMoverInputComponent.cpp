// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassMoverInputComponent.h"
#include "GameFramework/Actor.h"

void UMassMoverInputComponent::BeginPlay()
{
	Super::BeginPlay();

	// Sync to initial actor state
	DesiredRotation = GetOwner()->GetActorRotation().Quaternion();
	DesiredVelocity = FVector::Zero();
}

void UMassMoverInputComponent::ProduceInput_Implementation(int32 DeltaMs, FMoverInputCmdContext& OutInputCmd)
{
	FCharacterDefaultInputs& CharacterInputs = OutInputCmd.InputCollection.FindOrAddMutableDataByType<FCharacterDefaultInputs>();

	CharacterInputs.SetMoveInput(EMoveInputType::Velocity, DesiredVelocity);
	CharacterInputs.OrientationIntent = DesiredRotation.GetForwardVector();
}
