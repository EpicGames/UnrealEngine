// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassMoverInputTranslator.h"

#include "MassMoverInputComponent.h"
#include "MassExecutionContext.h"
#include "MassMovementFragments.h"
#include "MoverComponent.h"

UMassToMoverInputTranslator::UMassToMoverInputTranslator() : EntityQuery(*this)
{
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::Movement);
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::UpdateWorldFromMass;
}

void UMassToMoverInputTranslator::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FLightweightMoverInputWrapperFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FMassVelocityFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
}

void UMassToMoverInputTranslator::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
		{
			const TConstArrayView<FLightweightMoverInputWrapperFragment> MoverInputList = Context.GetFragmentView<FLightweightMoverInputWrapperFragment>();
			const TConstArrayView<FMassVelocityFragment> VelocityList = Context.GetFragmentView<FMassVelocityFragment>();
			const TConstArrayView<FTransformFragment> TransformList = Context.GetFragmentView<FTransformFragment>();

			const int32 NumEntities = Context.GetNumEntities();

			for (int32 i = 0; i < NumEntities; ++i)
			{
				// Do not pin since expensive. Assume the start to end execution of mass is within a frame & thus GC is not a concern.
				if (UMassMoverInputComponent* MoverInput = MoverInputList[i].MoverInputComponent.Get())
				{
					MoverInput->SetDesiredVelocity(VelocityList[i].Value);
					MoverInput->SetDesiredRotation(TransformList[i].GetTransform().GetRotation());
				}
			}
		});
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

UMoverInputToMassTranslator::UMoverInputToMassTranslator() : EntityQuery(*this)
{
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::SyncWorldToMass;
}

void UMoverInputToMassTranslator::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FMoverWrapperFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FMassVelocityFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadWrite);
}

void UMoverInputToMassTranslator::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
		{
			const TConstArrayView<FMoverWrapperFragment> MoverInputList = Context.GetFragmentView<FMoverWrapperFragment>();
			const TArrayView<FMassVelocityFragment> VelocityList = Context.GetMutableFragmentView<FMassVelocityFragment>();
			const TArrayView<FTransformFragment> TransformList = Context.GetMutableFragmentView<FTransformFragment>();

			const int32 NumEntities = Context.GetNumEntities();

			for (int32 i = 0; i < NumEntities; ++i)
			{
				// Do not pin since expensive. Assume the start to end execution of mass is within a frame & thus GC is not a concern.
				if (const UMoverComponent* Mover = MoverInputList[i].MoverComponent.Get())
				{
					VelocityList[i].Value = Mover->GetVelocity();
					TransformList[i].SetTransform(Mover->GetUpdatedComponentTransform());
				}
			}
		});
}
