// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoverInputTrait.h"

#include "MassEntityTemplateRegistry.h"
#include "MassEntityView.h"
#include "MassMovementFragments.h"
#include "MassMoverInputComponent.h"
#include "MassMoverInputTranslator.h"
#include "MoverComponent.h"
#include "VisualLogger/VisualLogger.h"

namespace
{
	template<typename T>
	T* AsComponent(UObject& Owner)
	{
		T* Component = nullptr;
		if (AActor* AsActor = Cast<AActor>(&Owner))
		{
			Component = AsActor->FindComponentByClass<T>();
		}
		else
		{
			Component = Cast<T>(&Owner);
		}

		UE_CVLOG_UELOG(Component == nullptr, &Owner, LogMass, Error, TEXT("Trying to extract %s from %s failed")
			, *T::StaticClass()->GetName(), *Owner.GetName());

		return Component;
	}
}

void UMoverInputTrait::BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const
{
	BuildContext.RequireFragment<FMassVelocityFragment>();
	BuildContext.RequireFragment<FTransformFragment>();
	BuildContext.AddFragment<FLightweightMoverInputWrapperFragment>();
	BuildContext.AddFragment<FMoverWrapperFragment>();

	BuildContext.GetMutableObjectFragmentInitializers().Add([this](UObject& Owner, const FMassEntityView& EntityView, const EMassTranslationDirection CurrentDirection)
		{
			if (UMassMoverInputComponent* MoverInputComponent = AsComponent<UMassMoverInputComponent>(Owner))
			{
				FLightweightMoverInputWrapperFragment& MoverInputWrapper = EntityView.GetFragmentData<FLightweightMoverInputWrapperFragment>();
				MoverInputWrapper.MoverInputComponent = MoverInputComponent;
			}

			// the entity is the authority
			if (CurrentDirection == EMassTranslationDirection::MassToActor)
			{
				// Do nothing
			}
			// actor is the authority
			else
			{
				if (UMoverComponent* MoverComponent = AsComponent<UMoverComponent>(Owner))
				{
					FMoverWrapperFragment& MoverWrapper = EntityView.GetFragmentData<FMoverWrapperFragment>();
					MoverWrapper.MoverComponent = MoverComponent;

					FTransformFragment& TransformFragment = EntityView.GetFragmentData<FTransformFragment>();
					TransformFragment.SetTransform(MoverComponent->GetUpdatedComponentTransform());
				}
			}
		});
}
