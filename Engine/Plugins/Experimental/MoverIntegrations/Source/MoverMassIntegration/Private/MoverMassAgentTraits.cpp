// Copyright Epic Games, Inc. All Rights Reserved.


#include "MoverMassAgentTraits.h"
#include "DefaultMovementSet/NavMoverComponent.h"
#include "MassMovementFragments.h"
#include "MassEntityTemplate.h"
#include "MassEntityTemplateRegistry.h"
#include "MoverMassTranslators.h"
#include "GameFramework/Actor.h"
#include "VisualLogger/VisualLogger.h"
#include "MassEntityView.h"

namespace UE::Mass::Mover::Private
{
	template<typename T>
	T* AsComponent(UObject& Owner)
	{
		T* Component = nullptr;
		if (AActor* AsActor = Cast<AActor>(&Owner))
		{
			Component = AsActor->FindComponentByClass<T>();
		}
		else
		{
			Component = Cast<T>(&Owner);
		}

		UE_CVLOG_UELOG(Component == nullptr, &Owner, LogMass, Error, TEXT("Trying to extract %s from %s failed")
					   , *T::StaticClass()->GetName(), *Owner.GetName());

		return Component;
	}
}


//----------------------------------------------------------------------//
//  UMoverMassAgentTrait
//----------------------------------------------------------------------//
void UMoverMassAgentTrait::BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const
{
	BuildContext.AddFragment<FNavMoverComponentWrapperFragment>();
	BuildContext.AddFragment<FMassVelocityFragment>();
	BuildContext.AddFragment<FTransformFragment>();
	
	BuildContext.GetMutableObjectFragmentInitializers().Add([this](UObject& Owner, FMassEntityView& EntityView, const EMassTranslationDirection CurrentDirection)
	{
		if (UNavMoverComponent* NavMoverComponent = UE::Mass::Mover::Private::AsComponent<UNavMoverComponent>(Owner))
		{
			FNavMoverComponentWrapperFragment& ComponentFragment = EntityView.GetFragmentData<FNavMoverComponentWrapperFragment>();
			ComponentFragment.Component = NavMoverComponent;

			FMassVelocityFragment& VelocityFragment = EntityView.GetFragmentData<FMassVelocityFragment>();

			// the entity is the authority
			if (CurrentDirection ==  EMassTranslationDirection::MassToActor)
			{
				NavMoverComponent->RequestDirectMove(VelocityFragment.Value, /*bForceMaxSpeed*/false);
			}
			// actor is the authority
			else
			{
				VelocityFragment.Value = NavMoverComponent->GetVelocityForNavMovement();
			}

			if (bSyncTransform)
			{
				FTransformFragment& TransformFragment = EntityView.GetFragmentData<FTransformFragment>();

				// the entity is the authority
				if (CurrentDirection ==  EMassTranslationDirection::MassToActor)
				{
					if (USceneComponent* UpdatedObjectAsSceneComponent = Cast<USceneComponent>(NavMoverComponent->GetUpdatedObject()))
					{
						// TODO: Mover also doesn't like setting transforms directly and may cause a warning about outside systems modifying the updated component
						UpdatedObjectAsSceneComponent->SetWorldTransform(TransformFragment.GetTransform());
					}
				}
				// actor is the authority
				else
				{
					if (const USceneComponent* UpdatedObjectAsSceneComponent = Cast<USceneComponent>(NavMoverComponent->GetUpdatedObject()))
					{
						TransformFragment.SetTransform(UpdatedObjectAsSceneComponent->GetComponentTransform());
					}
				}
			}
		}
	});

	if (EnumHasAnyFlags(SyncDirection, EMassTranslationDirection::ActorToMass) || BuildContext.IsInspectingData())
	{
		BuildContext.AddTranslator<UMassNavMoverToMassTranslator>();
	}

	if (EnumHasAnyFlags(SyncDirection, EMassTranslationDirection::MassToActor) || BuildContext.IsInspectingData())
	{
		BuildContext.AddTranslator<UMassToNavMoverTranslator>();
	}
}

//----------------------------------------------------------------------//
//  UMoverMassAgentOrientationSyncTrait
//----------------------------------------------------------------------//
void UMoverMassAgentOrientationSyncTrait::BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const
{
	BuildContext.RequireFragment<FNavMoverComponentWrapperFragment>();

	if (EnumHasAnyFlags(SyncDirection, EMassTranslationDirection::ActorToMass) 
		|| BuildContext.IsInspectingData())
	{
		BuildContext.AddTranslator<UMassNavMoverActorOrientationToMassTranslator>();
	}
	
	if (EnumHasAnyFlags(SyncDirection, EMassTranslationDirection::MassToActor)
		|| BuildContext.IsInspectingData())
	{
		BuildContext.AddTranslator<UMassOrientationToNavMoverActorOrientationTranslator>();
	}
}