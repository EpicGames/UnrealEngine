// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoverMassTranslators.h"
#include "DefaultMovementSet/NavMoverComponent.h"
#include "VisualLogger/VisualLogger.h"
#include "MassEntityManager.h"
#include "MassCommonTypes.h"
#include "MassExecutionContext.h"
#include "MassMovementFragments.h"
#include "MassSpawnerTypes.h"
#include "MassDebugger.h"

DECLARE_LOG_CATEGORY_EXTERN(LogMassMoverTranslator, Log, All);
DEFINE_LOG_CATEGORY(LogMassMoverTranslator);

DECLARE_LOG_CATEGORY_EXTERN(LogMassMoverDivergence, Log, All);
DEFINE_LOG_CATEGORY(LogMassMoverDivergence);

//----------------------------------------------------------------------//
//  UMassNavMoverToMassTranslator
//----------------------------------------------------------------------//
UMassNavMoverToMassTranslator::UMassNavMoverToMassTranslator()
	: EntityQuery(*this)
{
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::SyncWorldToMass;
	RequiredTags.Add<FMassNavMoverCopyToMassTag>();
}

void UMassNavMoverToMassTranslator::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	AddRequiredTagsToQuery(EntityQuery);
	EntityQuery.AddRequirement<FNavMoverComponentWrapperFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FMassVelocityFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FMassDesiredMovementFragment>(EMassFragmentAccess::ReadWrite);
}

void UMassNavMoverToMassTranslator::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
	{
		const TConstArrayView<FNavMoverComponentWrapperFragment> ComponentList = Context.GetFragmentView<FNavMoverComponentWrapperFragment>();
		const TArrayView<FTransformFragment> LocationList = Context.GetMutableFragmentView<FTransformFragment>();
		const TArrayView<FMassVelocityFragment> VelocityList = Context.GetMutableFragmentView<FMassVelocityFragment>();
		const TArrayView<FMassDesiredMovementFragment> DesiredMovementList = Context.GetMutableFragmentView<FMassDesiredMovementFragment>();

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			if (const UNavMoverComponent* AsMovementComponent = ComponentList[EntityIt].Component.Get())
			{
				LocationList[EntityIt].GetMutableTransform().SetLocation(AsMovementComponent->GetFeetLocation());
				VelocityList[EntityIt].Value = AsMovementComponent->GetVelocityForNavMovement();
				
				DesiredMovementList[EntityIt].DesiredMaxSpeedOverride = AsMovementComponent->GetMaxSpeedForNavMovement();	

#if WITH_MASSENTITY_DEBUG
				const bool bDisplayDebug = UE::Mass::Debug::IsDebuggingEntity(Context.GetEntity(EntityIt));
				if (bDisplayDebug)
				{
					const FVector& PreviousVelocity = VelocityList[EntityIt].DebugPreviousValue;
					const FVector ZOffset(0,0,5);
					const FVector& Location = LocationList[EntityIt].GetTransform().GetLocation() + ZOffset;

					constexpr FVector::FReal VelocityDeltaSquared = 1;
					if (FVector::DistSquared(PreviousVelocity, AsMovementComponent->GetVelocityForNavMovement()) > VelocityDeltaSquared)
					{
						// Draw expected and current velocities
						UE_VLOG_ARROW(this, LogMassMoverDivergence, Log, Location,
							Location + VelocityList[EntityIt].Value, FColor::Orange, TEXT("Current\nSpeed %.1f"), VelocityList[EntityIt].Value.Size());

						UE_VLOG_ARROW(this, LogMassMoverDivergence, Log, Location,
							Location + PreviousVelocity, FColor::Green, TEXT("Expected\nSpeed %.1f"), PreviousVelocity.Size());
					}
				}
#endif //WITH_MASSENTITY_DEBUG
			}
		}
	});
}

//----------------------------------------------------------------------//
//  UMassToNavMoverTranslator
//----------------------------------------------------------------------//
UMassToNavMoverTranslator::UMassToNavMoverTranslator()
	: EntityQuery(*this)
{
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::UpdateWorldFromMass;
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::Movement);
	RequiredTags.Add<FMassCopyToNavMoverTag>();
}

void UMassToNavMoverTranslator::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	AddRequiredTagsToQuery(EntityQuery);
	EntityQuery.AddRequirement<FNavMoverComponentWrapperFragment>(EMassFragmentAccess::ReadWrite);

#if WITH_MASSENTITY_DEBUG	
	EntityQuery.AddRequirement<FMassVelocityFragment>(EMassFragmentAccess::ReadWrite);
#else
	EntityQuery.AddRequirement<FMassVelocityFragment>(EMassFragmentAccess::ReadOnly);
#endif //WITH_MASSENTITY_DEBUG
	
	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
}

void UMassToNavMoverTranslator::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
	{
		const TArrayView<FNavMoverComponentWrapperFragment> ComponentList = Context.GetMutableFragmentView<FNavMoverComponentWrapperFragment>();

#if WITH_MASSGAMEPLAY_DEBUG
		const TArrayView<FMassVelocityFragment> VelocityList = Context.GetMutableFragmentView<FMassVelocityFragment>();
#else
		const TConstArrayView<FMassVelocityFragment> VelocityList = Context.GetFragmentView<FMassVelocityFragment>();
#endif //WITH_MASSGAMEPLAY_DEBUG

		const TConstArrayView<FTransformFragment> LocationList = Context.GetFragmentView<FTransformFragment>();
		
		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			if (UNavMoverComponent* AsMovementComponent = ComponentList[EntityIt].Component.Get())
			{
				FVector RequestedMove = VelocityList[EntityIt].Value;
				RequestedMove.Z = 0.0f;

#if WITH_MASSGAMEPLAY_DEBUG
				// Store requested velocity.
				VelocityList[EntityIt].DebugPreviousValue = RequestedMove;
#endif //WITH_MASSGAMEPLAY_DEBUG
			
				AsMovementComponent->RequestDirectMove(RequestedMove, /*bForceMaxSpeed=*/false);
				
#if WITH_MASSGAMEPLAY_DEBUG
				bool bDisplayDebug = UE::Mass::Debug::IsDebuggingEntity(Context.GetEntity(EntityIt));
				if (bDisplayDebug)
				{
					const FVector ActorLocation = AsMovementComponent->GetFeetLocation();
					const FVector EntityLocation = LocationList[EntityIt].GetTransform().GetLocation();
					
					UE_VLOG_ARROW(AsMovementComponent, LogMassMoverTranslator, Display, ActorLocation, ActorLocation + RequestedMove, FColor::Green, TEXT("Requested Move: %s"), *(RequestedMove.ToString()));
					UE_VLOG_SPHERE(AsMovementComponent, LogMassMoverTranslator, Display, EntityLocation, 5.0f, FColor::White, TEXT("EntityLocation: %s"), *(RequestedMove.ToString()));
					UE_VLOG_SPHERE(AsMovementComponent, LogMassMoverTranslator, Display, EntityLocation + RequestedMove, 5.0f, FColor::Blue, TEXT("EntityPrediction: %s"), *(RequestedMove.ToString()));
				}
#endif // WITH_MASSGAMEPLAY_DEBUG

			}
		}
	});
}


//----------------------------------------------------------------------//
//  UMassNavMoverActorOrientationToMassTranslator
//----------------------------------------------------------------------//
UMassNavMoverActorOrientationToMassTranslator::UMassNavMoverActorOrientationToMassTranslator()
: EntityQuery(*this)
{
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::SyncWorldToMass;
	RequiredTags.Add<FMassNavMoverActorOrientationCopyToMassTag>();
	bRequiresGameThreadExecution = true;
}

void UMassNavMoverActorOrientationToMassTranslator::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	AddRequiredTagsToQuery(EntityQuery);
	EntityQuery.AddRequirement<FNavMoverComponentWrapperFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadWrite);
}

void UMassNavMoverActorOrientationToMassTranslator::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
	{
		const TConstArrayView<FNavMoverComponentWrapperFragment> ComponentList = Context.GetFragmentView<FNavMoverComponentWrapperFragment>();
		const TArrayView<FTransformFragment> LocationList = Context.GetMutableFragmentView<FTransformFragment>();
		
		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			if (const UNavMoverComponent* AsNavMoverComponent = ComponentList[EntityIt].Component.Get())
			{
				if (const USceneComponent* UpdatedComponent = Cast<USceneComponent>(AsNavMoverComponent->GetUpdatedObject()))
				{
					LocationList[EntityIt].GetMutableTransform().SetRotation(UpdatedComponent->GetComponentTransform().GetRotation());
				}
			}
		}
	});
}

//----------------------------------------------------------------------//
//  UMassOrientationToNavMoverActorOrientationTranslator
//----------------------------------------------------------------------//
UMassOrientationToNavMoverActorOrientationTranslator::UMassOrientationToNavMoverActorOrientationTranslator()
: EntityQuery(*this)
{
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::UpdateWorldFromMass;
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::Movement);
	RequiredTags.Add<FMassOrientationCopyToNavMoverActorOrientationTag>();
	bRequiresGameThreadExecution = true;
}

void UMassOrientationToNavMoverActorOrientationTranslator::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	AddRequiredTagsToQuery(EntityQuery);
	EntityQuery.AddRequirement<FNavMoverComponentWrapperFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.RequireMutatingWorldAccess(); // due to mutating World by setting component rotation
}

void UMassOrientationToNavMoverActorOrientationTranslator::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
	{
		const TArrayView<FNavMoverComponentWrapperFragment> ComponentList = Context.GetMutableFragmentView<FNavMoverComponentWrapperFragment>();
		const TConstArrayView<FTransformFragment> TransformList = Context.GetFragmentView<FTransformFragment>();
		
		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			if (const UNavMoverComponent* AsNavMoverComponent = ComponentList[EntityIt].Component.Get())
			{
				if (USceneComponent* UpdatedComponent = Cast<USceneComponent>(AsNavMoverComponent->GetUpdatedObject()))
				{
					// TODO: Set OrientToMovement to true or false here - currently this isn't an option the Mover component but it should be
					// TODO: Mover also doesn't like setting rotation directly and may cause a warning about outside systems modifying the updated component
					const FTransformFragment& Transform = TransformList[EntityIt];
					UpdatedComponent->SetWorldRotation(Transform.GetTransform().GetRotation());
				}
			}
		}
	});
}
