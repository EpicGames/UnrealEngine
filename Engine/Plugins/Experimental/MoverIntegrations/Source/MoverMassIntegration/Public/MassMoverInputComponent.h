// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MoverSimulationTypes.h"
#include "MassMoverInputComponent.generated.h"

#define UE_API MOVERMASSINTEGRATION_API

/// Bridges a Mass entity to a Mover-based character actor. Each tick it produces an
/// FCharacterDefaultInputs frame (velocity + orientation intent), so this only works with
/// Mover modes that consume FCharacterDefaultInputs (UCharacterMoverComponent and similar).
UCLASS(MinimalAPI, BlueprintType, meta = (BlueprintSpawnableComponent))
class UMassMoverInputComponent : public UActorComponent, public IMoverInputProducerInterface
{
	GENERATED_BODY()

public:

	UE_API virtual void BeginPlay() override;

	const FVector& GetDesiredVelocity() const { return DesiredVelocity; }
	void SetDesiredVelocity(const FVector& NewDesiredVelocity) { DesiredVelocity = NewDesiredVelocity; }

	const FQuat& GetDesiredRotation() const { return DesiredRotation; }
	void SetDesiredRotation(const FQuat& NewDesiredRotation) { DesiredRotation = NewDesiredRotation; }

	UE_API virtual void ProduceInput_Implementation(int32 SimTimeMs, FMoverInputCmdContext& InputCmdResult) override;

protected:
	FVector DesiredVelocity = FVector::ZeroVector;
	FQuat DesiredRotation = FQuat::Identity;
};

#undef UE_API
