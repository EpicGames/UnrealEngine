// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassCommonFragments.h"
#include "MassEntityQuery.h"
#include "MassTranslator.h"

#include "MassMoverInputTranslator.generated.h"

#define UE_API MOVERMASSINTEGRATION_API

class UMassMoverInputComponent;
class UMoverComponent;

USTRUCT()
struct FLightweightMoverInputWrapperFragment : public FObjectWrapperFragment
{
	GENERATED_BODY()

	TWeakObjectPtr<UMassMoverInputComponent> MoverInputComponent;
};

USTRUCT()
struct FMoverWrapperFragment : public FObjectWrapperFragment
{
	GENERATED_BODY()

	TWeakObjectPtr<UMoverComponent> MoverComponent;
};

/// Mass -> Mover translator. Reads the entity's velocity and transform fragments and pushes the values
/// into the wrapped UMassMoverInputComponent so the Mover simulation produces the requested motion.
UCLASS(MinimalAPI)
class UMassToMoverInputTranslator : public UMassTranslator
{
	GENERATED_BODY()

public:
	UE_API UMassToMoverInputTranslator();

protected:
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery;
};

/// Mover -> Mass translator. Reads the wrapped UMoverComponent's velocity and updated transform back into
/// the entity's velocity and transform fragments so downstream Mass processors see the simulation output.
UCLASS(MinimalAPI)
class UMoverInputToMassTranslator : public UMassTranslator
{
	GENERATED_BODY()

public:
	UE_API UMoverInputToMassTranslator();

protected:
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery;
};

#undef UE_API
