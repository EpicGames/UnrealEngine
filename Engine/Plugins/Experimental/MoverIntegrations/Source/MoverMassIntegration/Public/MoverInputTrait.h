// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityTraitBase.h"
#include "MoverInputTrait.generated.h"

#define UE_API MOVERMASSINTEGRATION_API

/// Mass entity trait that wires a Mover-driven actor's input bridge into the entity. The trait adds the
/// fragments needed to read/write the desired velocity and transform, and depending on the sync direction
/// either captures the actor's current Mover transform or leaves the entity as the authority for movement.
UCLASS(MinimalAPI, BlueprintType, EditInlineNew, CollapseCategories, meta = (DisplayName = "Mover Input Trait"))
class UMoverInputTrait : public UMassEntityTraitBase
{
	GENERATED_BODY()

protected:
	UE_API virtual void BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const override;
};

#undef UE_API
