// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassActors/Public/MassAgentTraits.h"
#include "MoverMassAgentTraits.generated.h"

struct FMassEntityTemplateBuildContext;

#define UE_API MOVERMASSINTEGRATION_API

/**
 * The trait initializes the entity with a NavMoverComponent so Mover and Mass can communicate movement intent and velocity.
 * This trait also sets up necessary translators for these systems based off of Mass sync direction.
 * Note: This trait requires a NavMoverComponent and a MoverComponent to work properly
 */
UCLASS(MinimalAPI, BlueprintType, EditInlineNew, CollapseCategories, meta = (DisplayName = "Agent Mover Sync"))
class UMoverMassAgentTrait : public UMassAgentSyncTrait
{
	GENERATED_BODY()

protected:
	UE_API virtual void BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const override;

	/**
	 * Whether this trait should sync the transform of actor<->entity (based on translation direction)
	 * TODO: Currently Mover doesn't like outside modification of rotation and may throw a warning if Mover's transform gets set from the entity. It may also cause a rollback.
	 */
	UPROPERTY(EditAnywhere, Category = Mass)
	bool bSyncTransform = false;
};

/**
 * This trait sets up required translators for orientation syncing between Mover and Mass based off of Mass sync direction
 */
UCLASS(MinimalAPI, BlueprintType, EditInlineNew, CollapseCategories, meta = (DisplayName = "Agent Mover Orientation Sync"))
class UMoverMassAgentOrientationSyncTrait : public UMassAgentSyncTrait
{
	GENERATED_BODY()

protected:
	UE_API virtual void BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const override;
};

#undef UE_API
