// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

/**
 * Module for Mover-Mass integrations such as Mover specific Mass translators and Mass traits
 */
class FMoverMassIntegrationModule : public IModuleInterface
{
public:
    virtual void StartupModule() override;
    virtual void ShutdownModule() override;
};
