// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassCommonFragments.h"
#include "MassTranslator.h"
#include "MassEntityQuery.h"
#include "MassObserverProcessor.h"
#include "MoverMassTranslators.generated.h"

class UNavMoverComponent;

#define UE_API MOVERMASSINTEGRATION_API

USTRUCT()
struct FNavMoverComponentWrapperFragment : public FObjectWrapperFragment
{
	GENERATED_BODY()
	TWeakObjectPtr<UNavMoverComponent> Component;
};

USTRUCT()
struct FMassNavMoverCopyToMassTag : public FMassTag
{
	GENERATED_BODY()
};

/**
 * Translator in charge of going from Mover->Mass
 * Sets Mass fragments for keeping track of Location (feet location), Velocity and MaxSpeed
 */
UCLASS(MinimalAPI)
class UMassNavMoverToMassTranslator : public UMassTranslator
{
	GENERATED_BODY()

public:
	UE_API UMassNavMoverToMassTranslator();

protected:
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery;
};

USTRUCT()
struct FMassCopyToNavMoverTag : public FMassTag
{
	GENERATED_BODY()
};

/**
 * Translator in charge of going from Mass->Mover
 * Uses the NavMoverWrapperFragment to get a NavMoverComponent and request movement similarly to the PathFollowingComponent
 * See @UNavMoverComponent for details of how Mover consumes the move intent
 */
UCLASS(MinimalAPI)
class UMassToNavMoverTranslator : public UMassTranslator
{
	GENERATED_BODY()

public:
	UE_API UMassToNavMoverTranslator();

protected:
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery;
};

USTRUCT()
struct FMassNavMoverActorOrientationCopyToMassTag : public FMassTag
{
	GENERATED_BODY()
};

/**
 * Translator in charge of going from Mover->Mass for orientation
 * Modifies the transform fragment based off of Mover's rotation
 */
UCLASS(MinimalAPI)
class UMassNavMoverActorOrientationToMassTranslator : public UMassTranslator
{
	GENERATED_BODY()

public:
	UE_API UMassNavMoverActorOrientationToMassTranslator();

protected:
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery;
};

USTRUCT()
struct FMassOrientationCopyToNavMoverActorOrientationTag : public FMassTag
{
	GENERATED_BODY()
};

/**
 * Translator in charge of going from Mass->Mover for orientation
 * Modifies the UpdatedComponent of the NavMoverComponent based off of the entities transform fragment rotation
 * TODO: Currently Mover doesn't like outside modification of rotation and may throw a warning. It may also cause a rollback.
 */
UCLASS(MinimalAPI)
class UMassOrientationToNavMoverActorOrientationTranslator : public UMassTranslator
{
	GENERATED_BODY()

public:
	UE_API UMassOrientationToNavMoverActorOrientationTranslator();

protected:
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery;
};

#undef UE_API
